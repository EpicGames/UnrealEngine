// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class GeometryProcessingTarget : TargetRules
{
	public GeometryProcessingTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "GeometryProcessing";

		// Lean and mean
		bBuildDeveloperTools = false;

		bBuildWithEditorOnlyData = true;

		// Currently this app is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;
		bCompileAgainstApplicationCore = false;
		bCompileICU = false;

		// to enable tracing:
		// bEnableTrace = true;

		// This app is a console application, not a Windows app (sets entry point to main(), instead of WinMain())
		bIsBuildingConsoleApplication = true;
	}
}
