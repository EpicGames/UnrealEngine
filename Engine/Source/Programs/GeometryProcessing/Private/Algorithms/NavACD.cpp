// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompGeom/ConvexDecomposition3.h"
#include "MeshQueries.h"
#include "Util/ColorConstants.h"
#include "Utils/CommandUtils.h"

#include <fstream>
#include <iomanip>

using namespace UE::Geometry;

// The core navigation-driven approximate convex decomposition algorithm implementation
FConvexDecomposition3 RunNavACD(const FDynamicMesh3& Mesh, double MinRadiusFrac, double ToleranceFrac, bool bIgnoreUnreachableInternalSpace, 
	TArrayView<const FVector3d> CustomNavigablePositions = TArrayView<const FVector3d>());

// Algorithm runner (manages command line parameters, reading input and writing output)
DefineAlgorithm(NavACD)
{
	using namespace UE::CommandUtils;

	if (HasTag(TEXT("help")))
	{
		UE_LOGF(LogGeometryProcessing, Display, "Navigation-Driven Approximate Convex Decomposition arguments:");
		UE_LOGF(LogGeometryProcessing, Display, "-input: Path to input mesh");
		UE_LOGF(LogGeometryProcessing, Display, "-output: Path to output mesh; convex hulls will be assigned separate groups and colors");
		UE_LOGF(LogGeometryProcessing, Display, "-stats: If this tag is present, will output stats on a successful run (timings, hull counts)");
		UE_LOGF(LogGeometryProcessing, Display, "-protect_unreachable: If this tag is present, will protect unreachable space where the min radius sphere could fit.");
		UE_LOGF(LogGeometryProcessing, Display, "-r: Navigable space min radius parameter, as a fraction of the longest bounding box axis");
		UE_LOGF(LogGeometryProcessing, Display, "-t: Navigable space tolerance parameter, as a fraction of the longest bounding box axis");
		return true;
	}

	double MinRadiusFrac = RequireParam<double>("-r");
	double ToleranceFrac = RequireParam<double>("-t");

	bool bIgnoreUnreachableInternalSpace = !HasTag("protect_unreachable");

	FDynamicMesh3 Mesh = RequireInputMesh();
	FString OutputPath = RequireParam<FString>("-output");


	// Algorithm start time (not including file loading)
	double StartTime = FPlatformTime::Seconds();

	FConvexDecomposition3 ConvexDecomposition = RunNavACD(Mesh, MinRadiusFrac, ToleranceFrac, bIgnoreUnreachableInternalSpace);

	// Algorithm end time (not including file writing)
	double EndTime = FPlatformTime::Seconds();

	if (HasTag("stats"))
	{
		UE_LOGF(LogGeometryProcessing, Display, "Algorithm time (excluding file read/write): %f seconds", EndTime - StartTime);
		UE_LOGF(LogGeometryProcessing, Display, "Number of hulls used: %d", ConvexDecomposition.NumHulls());
	}

	// Write the decomposition to a single OBJ file, with a separate object name and vertex coloring per convex hull
	auto WriteDecomp = [](FConvexDecomposition3& Decomp, const FString& Path)
	{
		std::ofstream FileStream(TCHAR_TO_ANSI(*Path));
		if (!FileStream)
		{
			UE_LOGF(LogGeometryProcessing, Error, "Failed to open output path: %ls", *Path);
			Fail();
		}
		FileStream.precision(std::numeric_limits<double>::digits10);
		int32 LastV = 0;
		int32 ColorIdx = 0;
		auto DumpPart = [&FileStream, &LastV, &ColorIdx](const TArray<FIndex3i>& Tris, const TArray<FVector3d>& Verts)
		{
			const FVector Color = LinearColors::SelectColor<FVector>(ColorIdx++);
			FileStream << "o part" << ColorIdx << "\n";
			for (const FVector3d& V : Verts)
			{
				FileStream << "v " << V.X << " " << V.Y << " " << V.Z << " " << Color.X << " " << Color.Y << " " << Color.Z << "\n";
			}
			for (const FIndex3i& T : Tris)
			{
				FileStream << "f " << T.A + LastV + 1 << " " << T.C + LastV + 1 << " " << T.B + LastV + 1 << "\n";
			}
			LastV += Verts.Num();
			FileStream << "\n\n\n";
		};

		double SumVolume = 0;
		for (int32 CvxIdx = 0; CvxIdx < Decomp.NumHulls(); CvxIdx++)
		{
			TArray<FIndex3i> Tris = Decomp.GetTriangles(CvxIdx);
			TArray<FVector3d> Verts = Decomp.GetVertices<double>(CvxIdx);
			DumpPart(Tris, Verts);
		}
		FileStream.close();
	};

	WriteDecomp(ConvexDecomposition, OutputPath);

	return true;
}

FConvexDecomposition3 RunNavACD(const FDynamicMesh3& Mesh, double MinRadiusFrac, double ToleranceFrac, bool bIgnoreUnreachableInternalSpace, TArrayView<const FVector3d> CustomNavigablePositions)
{
	FAxisAlignedBox3d Bounds = Mesh.GetBounds();
	double MaxDim = Bounds.MaxDim();
	double UseMinRadius = MinRadiusFrac * MaxDim;
	double UseTolerance = ToleranceFrac * MaxDim;

	FConvexDecomposition3::FPreprocessMeshOptions PreprocessOptions;
	PreprocessOptions.bMergeEdges = true;
	PreprocessOptions.CustomPreprocess = [](FDynamicMesh3& ProcessMesh, const FAxisAlignedBox3d& Bounds) -> void
	{
		// for solid inputs, flip orientation if the initial volume is negative
		if (ProcessMesh.IsClosed())
		{
			double InitialVolume = TMeshQueries<FDynamicMesh3>::GetVolumeArea(ProcessMesh).X;
			if (InitialVolume < 0)
			{
				ProcessMesh.ReverseOrientation();
			}
		}
		// Note: If we add options to simplify the input mesh, should be applied here.
	};

	FConvexDecomposition3 ConvexDecomposition(Mesh, PreprocessOptions);
	const bool bIsSolid = ConvexDecomposition.IsInputSolid();
	ConvexDecomposition.bTreatAsSolid = bIsSolid;

	FNegativeSpaceSampleSettings NegativeSpaceSettings;
	NegativeSpaceSettings.ApplyDefaults();
	NegativeSpaceSettings.SampleMethod = FNegativeSpaceSampleSettings::ESampleMethod::NavigableVoxelSearch;
	NegativeSpaceSettings.bDeterministic = true;
	NegativeSpaceSettings.bRequireSearchSampleCoverage = true;
	NegativeSpaceSettings.bOnlyConnectedToHull = bIgnoreUnreachableInternalSpace;
	NegativeSpaceSettings.TargetNumSamples = 0;
	NegativeSpaceSettings.bAllowSamplesInsideMesh = !bIsSolid;

	NegativeSpaceSettings.ReduceRadiusMargin = UseTolerance;
	NegativeSpaceSettings.MinRadius = UseMinRadius;
	NegativeSpaceSettings.MinSpacing = 0;

	ConvexDecomposition.InitializeNegativeSpace(NegativeSpaceSettings, CustomNavigablePositions);

	ConvexDecomposition.MaxConvexEdgePlanes = 4;
	ConvexDecomposition.bSplitDisconnectedComponents = false;
	ConvexDecomposition.ConvexEdgeAngleMoreSamplesThreshold = 180; // a high value to disable this feature
	ConvexDecomposition.ThickenAfterHullFailure = FMath::Max(FMathd::ZeroTolerance, NegativeSpaceSettings.ReduceRadiusMargin * .01);
	constexpr int32 MaxAllowedSplits = 1000000; // more parts than any expected / reasonable decomposition
	for (int32 Split = 0; ; Split++)
	{
		int32 NumSplit = ConvexDecomposition.SplitWorst(false, -1, true, NegativeSpaceSettings.ReduceRadiusMargin * .5);

		if (NumSplit == 0)
		{
			break;
		}

		if (!ensureMsgf(Split < MaxAllowedSplits, TEXT("Convex decomposition split the input %d times; likely stuck in a loop"), Split))
		{
			break;
		}
	}

	ConvexDecomposition.FixHullOverlapsInNegativeSpace();

	int32 NumHullsBefore = ConvexDecomposition.NumHulls();
	constexpr double MinThicknessToleranceWorldSpace = 0;
	int32 NumMerged = ConvexDecomposition.MergeBest(-1, 0, MinThicknessToleranceWorldSpace, true);

	return ConvexDecomposition;
}