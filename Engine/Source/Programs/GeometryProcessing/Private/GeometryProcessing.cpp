// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryProcessing.h"

#include "Utils/CommandUtils.h"

#include "CompGeom/ExactPredicates.h"

#include "RequiredProgramMainCPPInclude.h"

DEFINE_LOG_CATEGORY(LogGeometryProcessing);

IMPLEMENT_APPLICATION(GeometryProcessing, "GeometryProcessing");


INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	ON_SCOPE_EXIT
	{ 
		LLM(FLowLevelMemTracker::Get().UpdateStatsPerFrame());
		RequestEngineExit(TEXT("Exiting"));
		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();
	};

	if (int32 Ret = GEngineLoop.PreInit(ArgC, ArgV))
	{
		return Ret;
	}

	// Initialize exact predicates (engine does this automatically on module init, but command line doesn't run the module init code)
	UE::Geometry::ExactPredicates::GlobalInit();

	// Run an algorithm as specified via the command line
	bool bSuccess = UE::CommandUtils::FAlgList::Run();

	return (int32)!bSuccess;
}
