// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/CommandUtils.h"

namespace UE::CommandUtils
{

	UE::Geometry::FDynamicMesh3 RequireInputMesh(const TCHAR* InputArg, const UE::MeshFileUtils::FLoadOBJSettings& Settings, bool bMustHaveFaces)
	{
		using namespace UE::MeshFileUtils;
		using namespace UE::Geometry;

		FString Path = RequireParam<FString>(InputArg);
		FDynamicMesh3 InputMesh;
		ELoadOBJStatus Status = LoadOBJ(TCHAR_TO_ANSI(*Path), InputMesh, Settings);
		if (Status != ELoadOBJStatus::Success)
		{
			UE_LOGF(LogGeometryProcessing, Error, "Failed to load OBJ from path: %ls", *Path);
			Fail();
		}
		if (bMustHaveFaces && InputMesh.TriangleCount() == 0)
		{
			UE_LOGF(LogGeometryProcessing, Error, "OBJ at path %ls had no triangles; cannot process.", *Path);
			Fail();
		}
		return InputMesh;
	}

	bool OutputResult(const UE::Geometry::FDynamicMesh3& Mesh, const TCHAR* OutputPathArg, const UE::MeshFileUtils::FWriteOBJSettings& Settings)
	{
		FString Path = RequireParam<FString>(OutputPathArg);
		bool bSuccess = UE::MeshFileUtils::WriteOBJ(TCHAR_TO_ANSI(*Path), Mesh, Settings);
		if (!bSuccess)
		{
			UE_LOGF(LogGeometryProcessing, Error, "Failed to write mesh output parameter %ls to path %ls", OutputPathArg, *Path);
		}
		return bSuccess;
	}

	static TMap<FString, TFunction<bool()>>& GetAlgorithms()
	{
		static TMap<FString, TFunction<bool()>> Algorithms;
		return Algorithms;
	}

	bool FAlgList::Register(FString Name, TFunction<bool()> Alg)
	{
		GetAlgorithms().Add(Name, Alg);
		return true;
	}

	bool FAlgList::Run()
	{
		TMap<FString, TFunction<bool()>>& Algorithms = GetAlgorithms();
		FString AlgName;
		if (FParse::Value(FCommandLine::Get(), TEXT("-alg"), AlgName))
		{
			TFunction<bool(void)>* AlgFunc = Algorithms.Find(AlgName);
			if (!AlgFunc)
			{
				UE_LOGF(LogGeometryProcessing, Error, "Unknown algorithm (%ls); -alg must be one of the following:", *AlgName);
				for (const auto& It : Algorithms)
				{
					UE_LOGF(LogGeometryProcessing, Error, "  -alg %ls", *It.Key);
				}
			}
			else
			{
				return (*AlgFunc)();
			}
		}
		else
		{
			UE_LOGF(LogGeometryProcessing, Error, "Must specify one of the following arguments to choose which algorithm to run:");
			for (const auto& It : Algorithms)
			{
				UE_LOGF(LogGeometryProcessing, Error, "  -alg %ls", *It.Key);
			}
		}
		return false;
	}
}