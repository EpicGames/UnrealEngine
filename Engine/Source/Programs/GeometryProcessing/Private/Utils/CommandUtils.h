// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "GeometryProcessing.h"
#include "OBJMeshUtil.h"

#include "Misc/CommandLine.h"
#include "Misc/Parse.h"

namespace UE::CommandUtils
{
	// For errors that we can't continue through -- immediately exits the program 
	inline void Fail()
	{
		FPlatformMisc::RequestExit(true);
	}

	// Read a required parameter from the command line, or exit if we cannot
	// Note that parameters are not required to have a preceding dash, so must be specified if desired (e.g., RequireParam("-input") if you expect "-input inputvalue")
	template<typename T>
	T RequireParam(const TCHAR* Arg)
	{
		T Value;
		if (FParse::Value(FCommandLine::Get(), Arg, Value))
		{
			return Value;
		}
		else
		{
			UE_LOGF(LogGeometryProcessing, Error, "Must specify a valid %ls parameter! Use -help to see algorithm parameters.", Arg);
			Fail();
			return T();
		}
	}
	template<typename T>
	T RequireParam(const char* Arg)
	{
		return RequireParam<T>(ANSI_TO_TCHAR(Arg));
	}

	// Request an optional parameter from the command line
	template<typename T>
	bool RequestParam(const TCHAR* Arg, T& Value)
	{
		return FParse::Value(FCommandLine::Get(), Arg, Value);
	}
	template<typename T>
	bool RequestParam(const char* Arg, T& Value)
	{
		return RequestParam<T>(ANSI_TO_TCHAR(Arg), Value);
	}

	// Check whether a tag is on the command line
	// Note the preceding "-" is implicit for this method, so HasTag("tag") will check for "-tag"
	inline bool HasTag(const TCHAR* Tag)
	{
		return FParse::Param(FCommandLine::Get(), Tag);
	}
	inline bool HasTag(const char* Tag)
	{
		return HasTag(ANSI_TO_TCHAR(Tag));
	}

	// Reads an OBJ mesh from path specified by the input parameter
	UE::Geometry::FDynamicMesh3 RequireInputMesh(const TCHAR* InputArg = TEXT("-input"), const UE::MeshFileUtils::FLoadOBJSettings& Settings = UE::MeshFileUtils::FLoadOBJSettings(), bool bMustHaveFaces = true);
	inline UE::Geometry::FDynamicMesh3 RequireInputMesh(const char* InputArg, const UE::MeshFileUtils::FLoadOBJSettings& Settings = UE::MeshFileUtils::FLoadOBJSettings(), bool bMustHaveFaces = true)
	{
		return RequireInputMesh(ANSI_TO_TCHAR(InputArg), Settings, bMustHaveFaces);
	}

	// Writes an OBJ mesh to the output parameter
	bool OutputResult(const UE::Geometry::FDynamicMesh3& Mesh, const TCHAR* OutputPathArg = TEXT("-output"), const UE::MeshFileUtils::FWriteOBJSettings& Settings = UE::MeshFileUtils::FWriteOBJSettings());
	inline bool OutputResult(const UE::Geometry::FDynamicMesh3& Mesh, const char* OutputPathArg, const UE::MeshFileUtils::FWriteOBJSettings& Settings = UE::MeshFileUtils::FWriteOBJSettings())
	{
		return OutputResult(Mesh, ANSI_TO_TCHAR(OutputPathArg), Settings);
	}

	// Struct to manage the list of known algorithms selectable with the -alg parameter
	struct FAlgList
	{
		static bool Register(FString Name, TFunction<bool()> Alg);
		static bool Run();
	};

}

// Macro to create an algorithm that will automatically register as an -alg parameter
#define DefineAlgorithm(Name) bool Alg_##Name(); static bool bAlgRegistered_##Name = UE::CommandUtils::FAlgList::Register(TEXT(#Name), &Alg_##Name); bool Alg_##Name()

